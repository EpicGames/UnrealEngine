{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "GPU Reshape Plugin",
	"Description": "GPU Reshape editor integration",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"EnabledByDefault": true,
	"SupportedTargetPlatforms":  [ "Win64" ],
	"Modules": [
		{
			"Name": "GPUReshape",
			"Type": "DeveloperTool",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64" ],
			"TargetDenyList": [ "Server" ]
		}
	]
}
