// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Linq;
using EpicGames.Core;
using UnrealBuildBase;

namespace UnrealBuildTool.Rules
{
	[SupportedPlatformGroups("Windows")]
	public class GPUReshape : ModuleRules
	{
		public GPUReshape(ReadOnlyTargetRules Target) : base(Target)
        {
			PublicDependencyModuleNames.AddAll(
				"InputDevice",
				"RenderCore"
			);

			PrivateDependencyModuleNames.AddAll(
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Projects",
				"RHI",
				"Json"
			);

			string[] SourcePaths = [
				"Source/Services/Loader/Include",
				"Source/Backends/DX12/Layer/Include",
				"Source/Backends/Vulkan/Layer/Include"
			];

			PrivateIncludePaths.AddRange(SourcePaths.Select(Path => DirectoryReference.Combine(
				Unreal.RootDirectory, "Engine", "Source", "ThirdParty", "GPUReshape", "Source", Path
			).FullName));
			
			if (Target.bBuildEditor)
			{
				DynamicallyLoadedModuleNames.Add("LevelEditor");

				PrivateDependencyModuleNames.AddAll(
					"Slate", 
					"SlateCore",
					"EditorFramework",
					"UnrealEd",
					"MainFrame",
					"GameProjectGeneration",
					"ToolMenus"
				);
			}
		}
	}
}
