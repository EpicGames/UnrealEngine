// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "GPUReshapeCommands.h"

#define LOCTEXT_NAMESPACE "GPUReshape"

void FGPUReshapeCommands::RegisterCommands()
{
	UI_COMMAND(OpenApp, "Open GPU-Reshape", "Open the GPU-Reshape application for debugging/instrumentation (Alt+F12)", EUserInterfaceActionType::Button, FInputChord(EKeys::F12, EModifierKey::Alt));
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_EDITOR
