// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if WITH_EDITOR

#include "GPUReshapeStyle.h"
#include "Framework/Commands/Commands.h"

class FGPUReshapeCommands : public TCommands<FGPUReshapeCommands>
{
public:
	FGPUReshapeCommands() : TCommands(TEXT("GPUReshape"), NSLOCTEXT("Contexts", "GPUReshape", "GPUReshape Plugin"), NAME_None, FGPUReshapeStyle::Get()->GetStyleSetName())
	{
		
	}

	/** TCommands overrides */
	virtual void RegisterCommands() override;

	TSharedPtr<class FUICommandInfo> OpenApp;
};

#endif // WITH_EDITOR
