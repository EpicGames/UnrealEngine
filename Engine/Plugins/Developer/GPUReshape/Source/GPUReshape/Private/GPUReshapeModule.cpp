// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPUReshapeModule.h"
#include "RenderingThread.h"
#include "UnrealClient.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "RHI.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/FileHelper.h"

#if WITH_EDITOR
#include "GPUReshapeCommands.h"
#include "GPUReshapeStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Kismet2/DebuggerCommands.h"
#include "ToolMenus.h"
#include "Editor.h"
#endif // WITH_EDITOR

#include "Services/Loader/Loader.h"

#if PLATFORM_WINDOWS
#	include "Windows/WindowsHWrapper.h"
#else // PLATFORM_WINDOWS
#	error Not supported
#endif // PLATFORM_WINDOWS

DEFINE_LOG_CATEGORY(LogGPUReshape);

static int32 GAutoAttachGPUReshape = 0;
static FAutoConsoleVariableRef CVarAutoAttachGPUReshape(
	TEXT("r.AutoAttachGPUReshape"),
	GAutoAttachGPUReshape,
	TEXT("Automatically loads GPU-Reshape on startup"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

void FGPUReshapeModule::StartupModule()
{
	const bool bCommandLineRequested =
		FParse::Param(FCommandLine::Get(), TEXT("AttachGRS")) ||
		FParse::Param(FCommandLine::Get(), TEXT("AttachReshape")) ||
		FParse::Param(FCommandLine::Get(), TEXT("AttachGPUReshape"));
	
	// Requested reshape?
	if (!GAutoAttachGPUReshape && !bCommandLineRequested)
	{
		return;
	}

	// Setup the loader and any backend state
	if (!FindAndInstallLoader())
	{
		return;
	}
	
	// Register console command
	OpenAppCommand = MakeUnique<FAutoConsoleCommand>(
		TEXT("GRS"), TEXT("Opens GPU Reshape"),
		FConsoleCommandDelegate::CreateRaw(this, &FGPUReshapeModule::OpenOrSwitchToApp)
	);

	// Post extension install
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FGPUReshapeModule::OnPostEngineInit);

	bBackendInitialized = true;
	UE_LOGF(LogGPUReshape, Log, "Initialized");
}

bool FGPUReshapeModule::InstallLoader(void* Handle)
{
	PFN_GRS_LOADER_INSTALL PFNInstall = reinterpret_cast<PFN_GRS_LOADER_INSTALL>(FPlatformProcess::GetDllExport(Handle, kPFNGRSLoaderInstallW));
	if (!PFNInstall)
	{
		UE_LOGF(LogGPUReshape, Error, "Failed to GPA '%ls'", kPFNGRSLoaderInstallW);
		return false;
	}

	// Symbol directory, Reshape is fairly Ansi based, for now at least
	FAnsiString ProjectSymbolDir = FAnsiString(FPaths::ProjectSavedDir() / TEXT("ShaderSymbols"));
	const char* ProjectSymbolDirPtr = *ProjectSymbolDir;

	// Setup the installation environment
	GRSLoaderInstallInfo info{};
	info.symbol.includeSubDirectories = true;
	info.symbol.pathCount = 1;
	info.symbol.paths = &ProjectSymbolDirPtr;

	// Handles general bootstrapping, layer injections, etc.
	return PFNInstall(&info);
}

void FGPUReshapeModule::CacheLoaderReservedToken(void* Handle)
{
	PFN_GRS_LOADER_GET_RESERVED_TOKEN PFNGetReservedToken = reinterpret_cast<PFN_GRS_LOADER_GET_RESERVED_TOKEN>(FPlatformProcess::GetDllExport(Handle, kPFNGRSLoaderGetReservedTokenW));
	if (!PFNGetReservedToken)
	{
		UE_LOGF(LogGPUReshape, Error, "Failed to GPA '%ls'", kPFNGRSLoaderGetReservedTokenW);
		return;
	}

	// Length of token
	uint32 tokenLength;
	PFNGetReservedToken(nullptr, &tokenLength);

	// Get token data
	char token[256]{};
	PFNGetReservedToken(token, &tokenLength);

	ReservedToken.Empty();
	ReservedToken.Append(token, tokenLength);
}

void FGPUReshapeModule::ShutdownModule()
{
	OpenAppCommand.Reset();
	
#if WITH_EDITOR
	FGPUReshapeStyle::Shutdown();
	FGPUReshapeCommands::Unregister();
#endif // WITH_EDITOR
}

bool FGPUReshapeModule::FindAndInstallLoader()
{
	if (GUsingNullRHI)
	{
		UE_LOGF(LogGPUReshape, Error, "Null-RHI not supported");
		return false;
	}
	
	// Allow binary overrides (useful for development)
	if (!FParse::Value(FCommandLine::Get(), TEXT("GRSPath"), GPUReshapePath))
	{
		// Default to the raytracing branch, until it's been merged back to dev
		FString Branch = "Raytracing";
		FParse::Value(FCommandLine::Get(), TEXT("GRSBranch"), Branch);
	
		// Default path, for now at least
		GPUReshapePath = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/GPUReshape/Win64") / Branch;
	}

	// Expected loader, handles all backends
	const FString GRSLoaderName = TEXT("GRS.Services.Loader.dll");

	// Valid path?
	const FString GRSLoaderPath = FPaths::Combine(*GPUReshapePath, *GRSLoaderName);
	if (!FPaths::FileExists(GRSLoaderPath))
	{
		UE_LOGF(LogGPUReshape, Error, "Loader not found: '%ls'", *GRSLoaderPath);
		return false;
	}

	// Try to load the lib
	void* Handle = FPlatformProcess::GetDllHandle(*GRSLoaderPath);
	if (!Handle)
	{
		UE_LOGF(LogGPUReshape, Error, "Failed to load library");
		return false;
	}

	// Try to install it
	if (!InstallLoader(Handle))
	{
		UE_LOGF(LogGPUReshape, Error, "Failed to install loader");
		return false;
	}
	

	UE_LOGF(LogGPUReshape, Display, "Installed loader: '%ls'", *GRSLoaderPath);

	// The initialized workspace is assigned a token, let's get it for later attaching
	CacheLoaderReservedToken(Handle);
	return true;
}

void FGPUReshapeModule::SwitchToApp()
{
	// Bring all relevant windows to the foreground
	EnumWindows([](HWND Window, LPARAM Param) -> BOOL
	{
		FGPUReshapeModule* This = reinterpret_cast<FGPUReshapeModule*>(Param);

		// Is child window?
		DWORD RemoteProcessID = 0;
		DWORD RemoteThreadID  = GetWindowThreadProcessId(Window, &RemoteProcessID);
		if (!RemoteThreadID || RemoteProcessID != This->GetAppGetProcessID())
		{
			return true;
		}

		// Attach to avoid anti-focus stealing
		AttachThreadInput(GetCurrentThreadId(), RemoteThreadID, true);
		SetForegroundWindow(Window);
		AttachThreadInput(GetCurrentThreadId(), RemoteThreadID, false);
		return true;
	}, reinterpret_cast<LPARAM>(this));
}

static bool CreateProjectSettingJson(FString& OutPath)
{
	TSharedPtr<FJsonObject> Object = MakeShared<FJsonObject>();
	
	// Targeting the editor
	Object->SetStringField(TEXT("ApplicationName"), TEXT("UnrealEditor.exe"));
	
	// Setup symbols
	TSharedPtr<FJsonObject> SymbolObject = MakeShared<FJsonObject>();
	{
		SymbolObject->SetArrayField(TEXT("Paths"), {
			MakeShared<FJsonValueString>(FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() / TEXT("ShaderSymbols")))
		});
		
		// Always sub-index
		SymbolObject->SetBoolField(TEXT("IncludeSubDirectories"), true);
	}
	Object->SetObjectField(TEXT("Symbol"), SymbolObject);
	
	// Setup symbols
	TSharedPtr<FJsonObject> SourceObject = MakeShared<FJsonObject>();
	{
		SourceObject->SetArrayField(TEXT("Paths"), {
			MakeShared<FJsonValueString>(FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT("Shaders")))
		});
		
		// Always sub-index
		SourceObject->SetBoolField(TEXT("IncludeSubDirectories"), true);
	}
	Object->SetObjectField(TEXT("Source"), SourceObject);
	
	// Serialize to text
	FString Contents;
	FJsonSerializer::Serialize(Object.ToSharedRef(), TJsonWriterFactory<>::Create(&Contents));
	
	// Try to write out
	OutPath = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("GPUReshapeSettings-"), TEXT(".json"));
	return FFileHelper::SaveStringToFile(Contents, *OutPath);
} 

void FGPUReshapeModule::OpenOrSwitchToApp()
{
	FString CommandLine;
	
	// Already running?
	if (AppProcHandle.IsValid() && FPlatformProcess::IsProcRunning(AppProcHandle))
	{
		SwitchToApp();
		return;
	}
	
	// Valid path?
	const FString AppPath = FPaths::Combine(*GPUReshapePath, TEXT("GPUReshape.exe"));
	if (!FPaths::FileExists(AppPath))
	{
		UE_LOGF(LogGPUReshape, Error, "App not found: '%ls'", *AppPath);
		return;
	}
	
	// Attach default settings
	if (FString SettingPath; CreateProjectSettingJson(SettingPath))
	{
		CommandLine += "apply-setting ";
		CommandLine += FString::Printf(TEXT("-json \"%s\" "), *SettingPath);
		CommandLine += " && ";
	}
	
	// Launch by attaching to the existing workspace
	CommandLine += "attach ";
	CommandLine += FString::Printf(TEXT("-pid %u "), FPlatformProcess::GetCurrentProcessId());
	CommandLine += FString::Printf(TEXT("-token %s "), *ReservedToken);
	CommandLine += FString::Printf(TEXT("-process %s "), FPlatformProcess::ExecutableName(false));

	// Try to create a job for shared lifetimes
	HANDLE Job = CreateJobObject(nullptr, nullptr);
	if (Job)
	{
		JOBOBJECT_EXTENDED_LIMIT_INFORMATION Info{};
		Info.BasicLimitInformation.LimitFlags = JOB_OBJECT_LIMIT_KILL_ON_JOB_CLOSE;
		if (!SetInformationJobObject(Job, JobObjectExtendedLimitInformation, &Info, sizeof(Info)))
		{
			UE_LOGF(LogGPUReshape, Error, "Failed to initialize job");
		}
	}
	else
	{
		UE_LOGF(LogGPUReshape, Error, "Failed to create job");
	}
	
	AppProcHandle = FPlatformProcess::CreateProc(
		*AppPath,
		*CommandLine,
		false,
		false,
		false,
		&AppProcessID,
		0,
		nullptr,
		nullptr,
		nullptr
	);

	// Group the process lifetimes
	if (Job && !AssignProcessToJobObject(Job, AppProcHandle.Get()))
	{
		UE_LOGF(LogGPUReshape, Error, "Failed to assign job");
	}
}

void FGPUReshapeModule::OnPostEngineInit()
{
#if WITH_EDITOR
	if (!FSlateApplication::IsInitialized() || IsRunningCommandlet())
	{
		return;
	}
	
	// Setup utilities
	FGPUReshapeStyle::Initialize();
	FGPUReshapeCommands::Register();

	if (!IsRunningGame())
	{
		// Support the new viewport toolbar
		if (FToolMenuOwnerScoped Scope(this); UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ViewportToolbar"))
		{
			FToolMenuSection& Section = Menu->FindOrAddSection("Right");

			// Add new section
			FToolMenuEntry& Entry = Section.AddMenuEntry(FGPUReshapeCommands::Get().OpenApp);
			Entry.ToolBarData.LabelOverride = FText::GetEmpty();
			Entry.InsertPosition.Position = EToolMenuInsertType::First;	
		}
	}

	if (GEditor)
	{
		check(FPlayWorldCommands::GlobalPlayWorldActions.IsValid());

		FPlayWorldCommands::GlobalPlayWorldActions->MapAction(
			FGPUReshapeCommands::Get().OpenApp,
			FExecuteAction::CreateLambda([this]
			{
				OpenOrSwitchToApp();
			}),
			FCanExecuteAction()
		);
	}
#endif // WITH_EDITOR
}

IMPLEMENT_MODULE(FGPUReshapeModule, GPUReshape)
