// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "HAL/PlatformProcess.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGPUReshape, Log, All);

class FGPUReshapeModule : public IModuleInterface
{
public:
	/** Open, or switch to if present, the app */
	void OpenOrSwitchToApp();

	/** Switch to the already open app */
	void SwitchToApp();

	/** Get the current process id */
	uint32 GetAppGetProcessID() const
	{
		return AppProcessID;
	}

	/** Is GPU Reshape initialized correctly? */
	bool IsInitialized() const
	{
		return bBackendInitialized;
	}
	
protected: /** IModuleInterface */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	/** Find the appropriate loader and install it */
	bool FindAndInstallLoader();
	
	/** Install the loader onto the current process */
	bool InstallLoader(void* Handle);

	/** Cache the reserved token for attaching  */
	void CacheLoaderReservedToken(void* Handle);

	/** Invoked for editor extensions */
	void OnPostEngineInit();

private:
	/** Commands */
	TUniquePtr<class FAutoConsoleCommand> OpenAppCommand;

	/** Bin path for tooling */
	FString GPUReshapePath;

	/** Cached token for attaching */
	FString ReservedToken;

	/** Current process handle */
	FProcHandle AppProcHandle;

	/** Current process id */
	uint32 AppProcessID = 0;

	/** Everything set up right? */
	bool bBackendInitialized = false;

#if WITH_EDITOR
	TSharedPtr<class FGPUReshapeEditorExtension> EditorExtension;
#endif // WITH_EDITOR
};
