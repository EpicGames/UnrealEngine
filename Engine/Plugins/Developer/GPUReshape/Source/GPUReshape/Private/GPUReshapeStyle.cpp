// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "GPUReshapeStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "HAL/FileManager.h"
#include "Interfaces/IPluginManager.h"
#include "GPUReshapeModule.h"

static TSharedPtr<FSlateStyleSet> GPUReshapeStyleSet;

/** Base path for all resources */
static FString GetAssetResourcePath()
{
	TSharedPtr<IPlugin> ThisPlugin = IPluginManager::Get().FindPlugin(TEXT("GPUReshape"));
	if (!ThisPlugin)
	{
		UE_LOGF(LogGPUReshape, Error, "Failed to find plugin");
		return "";
	}
	
	return ThisPlugin->GetBaseDir() / TEXT("Resources");
}

static FString GetAssetPath(const FString& RelativePath, const ANSICHAR* Extension)
{
	return GetAssetResourcePath() / RelativePath + Extension;
}

void FGPUReshapeStyle::Initialize()
{
	// Allow lazy init
	if (GPUReshapeStyleSet.IsValid())
	{
		return;
	}

	GPUReshapeStyleSet = MakeShareable(new FSlateStyleSet("GPUReshapeStyle"));

	// Always lives in the engine
	FString EngineResourceDir = FPaths::EnginePluginsDir() / TEXT("GPUReshape") / TEXT("Resources");
	GPUReshapeStyleSet->SetContentRoot(EngineResourceDir);
	GPUReshapeStyleSet->SetCoreContentRoot(EngineResourceDir);

	// Load resources
	GPUReshapeStyleSet->Set("GPUReshape.Icon", new FSlateImageBrush(GetAssetPath("IconNoFrame", ".png"), FVector2D(40.0f, 40.0f)));
	GPUReshapeStyleSet->Set("GPUReshape.OpenApp", new FSlateImageBrush(GetAssetPath("IconNoFrame16", ".png"), FVector2D(16.0f, 16.0f)));

	// Register
	FSlateStyleRegistry::RegisterSlateStyle(*GPUReshapeStyleSet.Get());
};

void FGPUReshapeStyle::Shutdown()
{
	if (GPUReshapeStyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*GPUReshapeStyleSet.Get());
		GPUReshapeStyleSet.Reset();
	}
}

TSharedPtr<ISlateStyle> FGPUReshapeStyle::Get()
{
	return GPUReshapeStyleSet;
}

#endif // WITH_EDITOR
