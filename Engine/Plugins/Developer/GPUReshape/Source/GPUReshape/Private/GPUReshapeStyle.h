// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Styling/ISlateStyle.h"

class FGPUReshapeStyle
{
public:
	/** Setup */
	static void Initialize();
	static void Shutdown();

	/** Get the shared style */
	static TSharedPtr<ISlateStyle> Get();
};

#endif // WITH_EDITOR
